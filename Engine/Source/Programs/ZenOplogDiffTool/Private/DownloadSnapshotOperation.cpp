// Copyright Epic Games, Inc. All Rights Reserved.

#include "DownloadSnapshotOperation.h"
#include "ZenOplogDiffLogging.h"

FDownloadSnapshot::FDownloadSnapshot(const UE::FZenSnapshotDescriptor& Descriptor, bool DownloadAsJson, FStringView OutputRoot)
	: SnapshotDescriptor(Descriptor)
	, OutputDirectory(OutputRoot)
	, DownloadManifestAsJson(DownloadAsJson)
{
}

IOplogDiffOperation::ERunningState FDownloadSnapshot::Run()
{
	UE::FDownloadOplogManifestResult DownloadResult = DownloadOplogManifest(SnapshotDescriptor, OutputDirectory, DownloadManifestAsJson);
	if (DownloadResult.Result == UE::FDownloadOplogManifestResult::EStatus::Error)
	{
		UE_LOGF(LogZenOplogDiffTool, Error, "Failed to download an oplog manifest!");
		return ERunningState::Error;
	}
	else
	{
		DownloadedManifest = DownloadResult.DownloadedFilename;
		UE_LOGF(LogZenOplogDiffTool, Display, "Snapshot downloaded to \"%ls\"", *DownloadResult.DownloadedFilename);
	}

	return ERunningState::Success;
}
