// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IZenOplogDiffOperation.h"
#include "Experimental/ZenOplogSnapshots.h"

// Downloads a snapshot from zen, using a snapshot descriptor
class FDownloadSnapshot : public IOplogDiffOperation
{
public:
	FDownloadSnapshot(const UE::FZenSnapshotDescriptor& Descriptor, bool DownloadAsJson, FStringView OutputRoot);
	virtual ~FDownloadSnapshot() = default;
	virtual ERunningState Run() override;

	UE::FZenSnapshotDescriptor SnapshotDescriptor;	// Used to identify the oplog to download
	FString OutputDirectory;			// Directory to write the manifests to (optional)
	bool DownloadManifestAsJson = false;

	FString DownloadedManifest;		// Local path to the manifest that was successfully downloaded
};
