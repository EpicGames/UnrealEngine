// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindSnapshotOperation.h"
#include "ZenOplogDiffLogging.h"

IOplogDiffOperation::ERunningState FFindSnapshot::Run()
{
	if (!PendingResult.IsValid())
	{
		PendingResult = UE::FindSnapshotDescriptorForBuild(FindProjectName, FindStream, FindPlatform, FindCommitID, bDisplayAllBuildsFound);
	}
	else if(PendingResult.IsReady())
	{
		FoundSnapshot = PendingResult.Get();
		if (FoundSnapshot)
		{
			UE_LOGF(LogZenOplogDiffTool, Display, "Found matching build:\nHost: %ls\nNamespace: %ls\nBucket: %ls\nBuildID: %ls",
				*FoundSnapshot->CloudHost,
				*FoundSnapshot->Namespace,
				*FoundSnapshot->Bucket,
				*FoundSnapshot->BuildID);
			return ERunningState::Success;
		}
		else
		{
			UE_LOGF(LogZenOplogDiffTool, Error, "Failed to find a matching build");
			return ERunningState::Error;
		}
	}
	return ERunningState::Running;
}
