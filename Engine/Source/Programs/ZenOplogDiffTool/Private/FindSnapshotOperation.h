// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IZenOplogDiffOperation.h"
#include "Experimental/ZenOplogSnapshots.h"

// Uses the zen build service to lookup a snapshot descriptor for a particular build
class FFindSnapshot : public IOplogDiffOperation
{
public:
	virtual ~FFindSnapshot() = default;
	virtual ERunningState Run() override;

	// Find a snapshot based on these parameters
	FString FindProjectName;
	FString FindStream;
	FString FindPlatform;
	FString FindCommitID;
	bool bDisplayAllBuildsFound = false;

	TSharedFuture<TOptional<UE::FZenSnapshotDescriptor>> PendingResult;
	TOptional<UE::FZenSnapshotDescriptor> FoundSnapshot;
};
