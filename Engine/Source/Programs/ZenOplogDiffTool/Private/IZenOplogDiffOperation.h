// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"

// Represents an operation that can run continuously.
class IOplogDiffOperation
{
public:
	virtual ~IOplogDiffOperation() = default;

	enum class ERunningState
	{
		Running,
		Success,
		Error
	};

	virtual ERunningState Run() = 0;					// Continuously called until it returns Success or Error
	TFunction<void(IOplogDiffOperation&)> OnComplete;	// Parameter = reference to operation that just completed. Should be called when Run returns Success
};
