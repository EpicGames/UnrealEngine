// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadSnapshotDescriptorOperation.h"
#include "ZenOplogDiffLogging.h"

FLoadSnapshotDescriptor::FLoadSnapshotDescriptor(FStringView SnapshotJson)
	: SnapshotJsonPath(SnapshotJson)
{
}

IOplogDiffOperation::ERunningState FLoadSnapshotDescriptor::Run()
{
	UE_LOGF(LogZenOplogDiffTool, Verbose, "Parsing snapshot descriptors from %ls", *SnapshotJsonPath);
	LoadedSnapshots = UE::ParseSnapshotDescriptorFromJson(SnapshotJsonPath);
	return LoadedSnapshots.Num() > 0 ? ERunningState::Success : ERunningState::Error;
}
