// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IZenOplogDiffOperation.h"
#include "Experimental/ZenOplogSnapshots.h"

// Load a snapshot descriptor from horde build artifact json
class FLoadSnapshotDescriptor : public IOplogDiffOperation
{
public:
	FLoadSnapshotDescriptor(FStringView SnapshotJson);
	virtual ~FLoadSnapshotDescriptor() = default;
	virtual ERunningState Run() override;

	FString SnapshotJsonPath;		// Path to the file to load
	TArray<UE::FZenSnapshotDescriptor> LoadedSnapshots;		// Snapshot json may contain more than one build
};
