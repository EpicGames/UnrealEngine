// Copyright Epic Games, Inc. All Rights Reserved.

#include "OplogManifestDiffOperation.h"
#include "ZenOplogDiffLogging.h"
#include "Experimental/DiffCompactBinary.h"
#include "Experimental/ZenOplogDiff.h"
#include "Async/TaskGraphInterfaces.h"

FOplogManifestDiff::FOplogManifestDiff(FString Manifest1, FString Manifest2)
	: ManifestToDiff1(Manifest1)
	, ManifestToDiff2(Manifest2)
{
}

IOplogDiffOperation::ERunningState FOplogManifestDiff::Run()
{
	if (!LoadManifests())
	{
		UE_LOGF(LogZenOplogDiffTool, Error, "Failed to load manifests");
		return ERunningState::Error;
	}
	UE_LOGF(LogZenOplogDiffTool, Display, "Diffing oplogs...");
	DiffResults = DiffManifests(Manifest1, Manifest2);

	return ERunningState::Success;
}

bool FOplogManifestDiff::LoadManifests()
{
	UE_LOGF(LogZenOplogDiffTool, Display, "Loading oplogs...");
	FGraphEventArray LoadingTasks;
	TAtomic<bool> bLoadedOk = true;
	LoadingTasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([this, &bLoadedOk]()
	{
		UE::FLoadOplogManifestResult LoadedManifest1 = UE::LoadOplogManifestFromFile(ManifestToDiff1);
		if (LoadedManifest1.Result != UE::FLoadOplogManifestResult::EStatus::Ok)
		{
			UE_LOGF(LogZenOplogDiffTool, Error, "Failed to load manifest from file %ls", *ManifestToDiff1);
			bLoadedOk = false;
		}
		else
		{
			Manifest1 = MoveTemp(*LoadedManifest1.Manifest);
		}
	}));
	LoadingTasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([this, &bLoadedOk]()
	{
		UE::FLoadOplogManifestResult LoadedManifest2 = UE::LoadOplogManifestFromFile(ManifestToDiff2);
		if (LoadedManifest2.Result != UE::FLoadOplogManifestResult::EStatus::Ok)
		{
			UE_LOGF(LogZenOplogDiffTool, Error, "Failed to load manifest from file %ls", *ManifestToDiff2);
			bLoadedOk = false;
		}
		else
		{
			Manifest2 = MoveTemp(*LoadedManifest2.Manifest);
		}
	}));
	FTaskGraphInterface::Get().WaitUntilTasksComplete(LoadingTasks);

	return bLoadedOk;
}
