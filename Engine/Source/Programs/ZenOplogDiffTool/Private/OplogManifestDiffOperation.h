// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IZenOplogDiffOperation.h"
#include "Experimental/ZenOplogManifest.h"
#include "Experimental/ZenOplogDiff.h"

class FOplogManifestDiff : public IOplogDiffOperation
{
public:
	FOplogManifestDiff(FString Manifest1, FString Manifest2);
	virtual ~FOplogManifestDiff() = default;
	virtual ERunningState Run() override;

	// Paths to local manifest files
	FString ManifestToDiff1;
	FString ManifestToDiff2;

	UE::FOplogManifest Manifest1;
	UE::FOplogManifest Manifest2;

	// Results of the diff
	UE::FOplogDiffResults DiffResults;

private:
	bool LoadManifests();
};
