// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputDiffResultsOperation.h"
#include "ZenOplogDiffLogging.h"
#include "Experimental/DiffCompactBinary.h"
#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/JsonWriter.h"

struct FOutputDiffResults::FDiffSummary
{
	int64 IdenticalPackageDataTotalSize = 0;
	int64 IdenticalBulkDataTotalSize = 0;
	int64 NewPackageDataTotalSize = 0;
	int64 NewBulkDataTotalSize = 0;
	int64 DeletedPackageDataTotalSize = 0;
	int64 DeletedBulkDataTotalSize = 0;
	int64 ChangedPackageDataTotalSize = 0;
	int64 ChangedBulkDataTotalSize = 0;

	void Output(TSharedRef<TJsonWriter<>>& Writer) const
	{
		Writer->WriteObjectStart("Summary");
		Writer->WriteValue("IdenticalPackageDataSize", IdenticalPackageDataTotalSize);
		Writer->WriteValue("IdenticalBulkDataSize", IdenticalBulkDataTotalSize);
		Writer->WriteValue("NewPackageDataTotalSize", NewPackageDataTotalSize);
		Writer->WriteValue("NewBulkDataTotalSize", NewBulkDataTotalSize);
		Writer->WriteValue("DeletedPackageDataTotalSize", DeletedPackageDataTotalSize);
		Writer->WriteValue("DeletedBulkDataTotalSize", DeletedBulkDataTotalSize);
		Writer->WriteValue("ChangedPackageDataTotalSize", ChangedPackageDataTotalSize);
		Writer->WriteValue("ChangedBulkDataTotalSize", ChangedBulkDataTotalSize);
		Writer->WriteObjectEnd();
	}

	void Output(FCbWriter& Writer) const
	{
		Writer.BeginObject("Summary");
		Writer.AddInteger("IdenticalPackageDataSize", IdenticalPackageDataTotalSize);
		Writer.AddInteger("IdenticalBulkDataSize", IdenticalBulkDataTotalSize);
		Writer.AddInteger("NewPackageDataTotalSize", NewPackageDataTotalSize);
		Writer.AddInteger("NewBulkDataTotalSize", NewBulkDataTotalSize);
		Writer.AddInteger("DeletedPackageDataTotalSize", DeletedPackageDataTotalSize);
		Writer.AddInteger("DeletedBulkDataTotalSize", DeletedBulkDataTotalSize);
		Writer.AddInteger("ChangedPackageDataTotalSize", ChangedPackageDataTotalSize);
		Writer.AddInteger("ChangedBulkDataTotalSize", ChangedBulkDataTotalSize);
		Writer.EndObject();
	}

	void operator+=(const FDiffSummary& Other)
	{
		IdenticalPackageDataTotalSize += Other.IdenticalPackageDataTotalSize;
		IdenticalBulkDataTotalSize += Other.IdenticalBulkDataTotalSize;
		NewPackageDataTotalSize += Other.NewPackageDataTotalSize;
		NewBulkDataTotalSize += Other.NewBulkDataTotalSize;
		DeletedPackageDataTotalSize += Other.DeletedPackageDataTotalSize;
		DeletedBulkDataTotalSize += Other.DeletedBulkDataTotalSize;
		ChangedPackageDataTotalSize += Other.ChangedPackageDataTotalSize;
		ChangedBulkDataTotalSize += Other.ChangedBulkDataTotalSize;
	}
};

FOutputDiffResults::FOutputDiffResults(UE::FOplogManifest&& Oplog1, UE::FOplogManifest&& Oplog2, UE::FOplogDiffResults&& Results)
	: Manifest1(MoveTemp(Oplog1))
	, Manifest2(MoveTemp(Oplog2))
	, DiffResults(MoveTemp(Results))
{
}

void FOutputDiffResults::LogOutput()
{
	const bool bDisplaySummary = LoggerOptions == ELogOptions::Summary;
	if (bDisplaySummary)
	{
		UE_LOGF(LogZenOplogDiffTool, Display, "Identical Ops: %d", DiffResults.IdenticalOps.Num());
		UE_LOGF(LogZenOplogDiffTool, Display, "New Ops: %d", DiffResults.OpsMissingInManifest1.Num());
		UE_LOGF(LogZenOplogDiffTool, Display, "Deleted Ops: %d", DiffResults.OpsMissingInManifest2.Num());
		UE_LOGF(LogZenOplogDiffTool, Display, "Changed Ops With Different Output: %d", DiffResults.ChangedOpsWithOutputDifferences.Num());
		UE_LOGF(LogZenOplogDiffTool, Display, "Changed Ops With Same: %d", DiffResults.ChangedOpsWithSameOutput.Num());
	}
}

FOutputDiffResults::FDiffSummary FOutputDiffResults::SummarizeDiffResults()
{
	FDiffSummary TotalSummary;
	auto SimpleSummary = [](const UE::FOplogManifest::FOp& TheOp, int64& PackageDataSize, int64& BulkDataSize)
	{
		for (const UE::FOplogManifest::FOp::FPackageData& PackageData : TheOp.GetPackageDatas())
		{
			PackageDataSize += PackageData.Size;
		}
		for (const UE::FOplogManifest::FOp::FBulkData& BulkData : TheOp.GetBulkDatas())
		{
			BulkDataSize += BulkData.Size;
		}
	};

	TArray<FDiffSummary> Summaries;
	ParallelForWithTaskContext(Summaries, DiffResults.IdenticalOps.Num(), [&SimpleSummary, this](FDiffSummary& Context, int32 Index)
	{
		const UE::FOplogManifest::FOp& TheOp = Manifest1.Ops[Manifest1.OpKeyToIndex[DiffResults.IdenticalOps[Index]]];
		SimpleSummary(TheOp, Context.IdenticalPackageDataTotalSize, Context.IdenticalBulkDataTotalSize);
	});
	for (const FDiffSummary& Summary : Summaries)
	{
		TotalSummary += Summary;
	}
	ParallelForWithTaskContext(Summaries, DiffResults.ChangedOpsWithSameOutput.Num(), [&SimpleSummary, this](FDiffSummary& Context, int32 Index)
	{
		const UE::FOplogManifest::FOp& TheOp = Manifest1.Ops[Manifest1.OpKeyToIndex[DiffResults.ChangedOpsWithSameOutput[Index].Key]];
		SimpleSummary(TheOp, Context.IdenticalPackageDataTotalSize, Context.IdenticalBulkDataTotalSize);
	});
	for (const FDiffSummary& Summary : Summaries)
	{
		TotalSummary += Summary;
	}
	ParallelForWithTaskContext(Summaries, DiffResults.OpsMissingInManifest1.Num(), [&SimpleSummary, this](FDiffSummary& Context, int32 Index)
	{
		const UE::FOplogManifest::FOp& TheOp = Manifest2.Ops[Manifest2.OpKeyToIndex[DiffResults.OpsMissingInManifest1[Index]]];
		SimpleSummary(TheOp, Context.NewPackageDataTotalSize, Context.NewBulkDataTotalSize);
	});
	for (const FDiffSummary& Summary : Summaries)
	{
		TotalSummary += Summary;
	}
	ParallelForWithTaskContext(Summaries, DiffResults.OpsMissingInManifest2.Num(), [&SimpleSummary, this](FDiffSummary& Context, int32 Index)
	{
		const UE::FOplogManifest::FOp& TheOp = Manifest1.Ops[Manifest1.OpKeyToIndex[DiffResults.OpsMissingInManifest2[Index]]];
		SimpleSummary(TheOp, Context.DeletedPackageDataTotalSize, Context.DeletedBulkDataTotalSize);
	});
	for (const FDiffSummary& Summary : Summaries)
	{
		TotalSummary += Summary;
	}
	ParallelForWithTaskContext(Summaries, DiffResults.ChangedOpsWithOutputDifferences.Num(), [&SimpleSummary, this](FDiffSummary& Context, int32 Index)
	{
		const UE::FOplogManifest::FOp& TheOp = Manifest2.Ops[Manifest2.OpKeyToIndex[DiffResults.ChangedOpsWithOutputDifferences[Index].Key]];
		SimpleSummary(TheOp, Context.ChangedPackageDataTotalSize, Context.ChangedBulkDataTotalSize);
	});
	for (const FDiffSummary& Summary : Summaries)
	{
		TotalSummary += Summary;
	}
	return TotalSummary;
}

bool FOutputDiffResults::OutputCompactBinary(const FDiffSummary& Summary, UE::EOutputManifestDiffOptions Flags)
{
	FCbWriter Writer;
	Writer.BeginObject();
	OutputManifestDiffResultsToCompactBinary(Writer, DiffResults, Flags);
	Summary.Output(Writer);
	Writer.EndObject();
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*OutputPath));
	if (Ar && !Ar->IsError())
	{
		Writer.Save(*Ar);
		return !Ar->IsError();
	}
	else
	{
		return false;
	}
}

bool FOutputDiffResults::OutputJson(const FDiffSummary& Summary, UE::EOutputManifestDiffOptions Flags)
{
	FString OutputString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory <>::Create(&OutputString, 0);
	JsonWriter->WriteObjectStart();
	OutputManifestDiffResultsToJson(JsonWriter, DiffResults, Flags);
	Summary.Output(JsonWriter);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	if (!FFileHelper::SaveStringToFile(OutputString, *OutputPath))
	{
		UE_LOGF(LogZenOplogDiffTool, Error, "Failed to write diff results to '%ls'", *OutputPath);
		return false;
	}
	return true;
}

IOplogDiffOperation::ERunningState FOutputDiffResults::Run()
{
	UE_LOGF(LogZenOplogDiffTool, Display, "Sorting diff results");
	UE::SortDiffResults(DiffResults);
	FDiffSummary DiffSummary = SummarizeDiffResults();

	LogOutput();
	
	if (OutputPath.Len() > 0)
	{
		const FString FileExtension = FPaths::GetExtension(OutputPath).ToLower();
		UE::EOutputManifestDiffOptions OutputFlags = UE::EOutputManifestDiffOptions::None;
		if (OutputOptions == EOutputOptions::OutputChangedValues)
		{
			OutputFlags |= UE::EOutputManifestDiffOptions::OutputDifferences;
		}
		
		bool WriteSucceeded = false;
		if (FileExtension == TEXT("json"))
		{
			UE_LOGF(LogZenOplogDiffTool, Display, "Writing results to %ls", *OutputPath);
			WriteSucceeded = OutputJson(DiffSummary, OutputFlags);
		}
		else if (FileExtension == TEXT("cb"))
		{
			UE_LOGF(LogZenOplogDiffTool, Display, "Writing results to %ls", *OutputPath);
			WriteSucceeded = OutputCompactBinary(DiffSummary, OutputFlags);
		}
		else
		{
			UE_LOGF(LogZenOplogDiffTool, Error, "Unsupported diff output path '%ls'. Expected .cb or .json extension", *OutputPath);
			return ERunningState::Error;
		}

		if(!WriteSucceeded)
		{
			UE_LOGF(LogZenOplogDiffTool, Warning, "Failed to write results to file");
			return ERunningState::Error;
		}
	}
	
	return ERunningState::Success;
}
