// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IZenOplogDiffOperation.h"
#include "Experimental/ZenOplogDiff.h"
#include "Experimental/ZenOplogManifest.h"

// Takes ownership of diff results + outputs them to a file and/or the log
class FOutputDiffResults : public IOplogDiffOperation
{
public:
	enum class ELogOptions
	{
		None,
		Summary
	};

	enum class EOutputOptions
	{
		None,
		OutputChangedValues
	};

	FOutputDiffResults(UE::FOplogManifest&& Oplog1, UE::FOplogManifest&& Oplog2, UE::FOplogDiffResults&& DiffResults);
	virtual ~FOutputDiffResults() = default;
	virtual ERunningState Run() override;

	FString OutputPath;
	ELogOptions LoggerOptions = ELogOptions::None;
	EOutputOptions OutputOptions = EOutputOptions::None;

	UE::FOplogManifest Manifest1;
	UE::FOplogManifest Manifest2;
	UE::FOplogDiffResults DiffResults;

private:
	struct FDiffSummary;

	void LogOutput();
	bool OutputCompactBinary(const FDiffSummary& Summary, UE::EOutputManifestDiffOptions Flags);
	bool OutputJson(const FDiffSummary& Summary, UE::EOutputManifestDiffOptions Flags);
	FDiffSummary SummarizeDiffResults();
};
