// Copyright Epic Games, Inc. All Rights Reserved.

#include "RequiredProgramMainCPPInclude.h"
#include "ZenOplogDiffLogging.h"
#include "IZenOplogDiffOperation.h"
#include "DownloadSnapshotOperation.h"
#include "OplogManifestDiffOperation.h"
#include "OutputDiffResultsOperation.h"
#include "FindSnapshotOperation.h"
#include "LoadSnapshotDescriptorOperation.h"
#include "CoreGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "Experimental/ZenOplogDiff.h"

DEFINE_LOG_CATEGORY(LogZenOplogDiffTool);

IMPLEMENT_APPLICATION(ZenOplogDiffTool, "ZenOplogDiffTool");

void PrintHelp()
{
	UE_LOGF(LogZenOplogDiffTool, Display, "Zen Oplog Diff Utility");
	UE_LOGF(LogZenOplogDiffTool, Display, "Usage:");
	UE_LOGF(LogZenOplogDiffTool, Display, "-find-snapshot - Find a snapshot using the build service");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-project=<name> - Project Name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-stream=<name> - Stream Name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-platform=<name> - Platform name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-commit=<cl> - Commit ID / Changelist");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-displaybuilds - Output the list of namespaces, buckets, and commits found (optional)");
	UE_LOGF(LogZenOplogDiffTool, Display, "-download-snapshot - Download an oplog snapshot from cloud storage");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-output-directory=<path> - Base directory to download to (optional)");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-as-json - Download the manifest as json (optional)");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-snapshot-json=<path> - Path to an exported zen snapshot descriptor (from horde)");
	UE_LOG(LogZenOplogDiffTool, Display, TEXT("\t\tOR"));
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-project=<name> - Project Name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-stream=<name> - Stream Name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-platform=<name> - Platform name");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-commit=<cl> - Commit ID / Changelist");
	UE_LOGF(LogZenOplogDiffTool, Display, "-diff-oplog -oplog1=<path> -oplog2=<path> - Diff 2 oplogs and output the results to a file/stdout");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-results-path=<path> - Output the full results to a file");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-output-values - Outputs the values of any property that was changed (optional)");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t-log-output=<option> - Optional logging to stdout");
	UE_LOGF(LogZenOplogDiffTool, Display, "\t\tsummary - Outputs counts of identical, changed, and missing oplogs");
}

void CreateFindSnapshotCommands(TQueue<TUniquePtr<IOplogDiffOperation>>& Operations)
{
	FString ProjectName;
	FString PlatformName;
	FString StreamName;
	FString CommitID;
	if (FParse::Value(FCommandLine::Get(), TEXT("-project="), ProjectName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-stream="), StreamName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-platform="), PlatformName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-commit="), CommitID))
	{
		TUniquePtr<FFindSnapshot> Operation = MakeUnique<FFindSnapshot>();
		Operation->FindPlatform = PlatformName;
		Operation->FindProjectName = ProjectName;
		Operation->FindStream = StreamName;
		Operation->FindCommitID = CommitID;
		Operation->bDisplayAllBuildsFound = FParse::Param(FCommandLine::Get(), TEXT("-displaybuilds"));
		Operations.Enqueue(MoveTemp(Operation));
	}
}

FOutputDiffResults::ELogOptions GetDiffLogOptionsFromCmdLine()
{
	FString LogOptionString;
	FOutputDiffResults::ELogOptions Logging = FOutputDiffResults::ELogOptions::None;
	FParse::Value(FCommandLine::Get(), TEXT("-log-output="), LogOptionString);
	if (LogOptionString == "summary")
	{
		Logging = FOutputDiffResults::ELogOptions::Summary;
	}
	return Logging;
}

FOutputDiffResults::EOutputOptions GetDiffOutputOptionsFromCmdLine()
{
	FOutputDiffResults::EOutputOptions OutputOption = FOutputDiffResults::EOutputOptions::None;
	if (FParse::Param(FCommandLine::Get(), TEXT("-output-values")))
	{
		OutputOption = FOutputDiffResults::EOutputOptions::OutputChangedValues;
	}
	return OutputOption;
}

void CreateManifestDiffCommands(TQueue<TUniquePtr<IOplogDiffOperation>>& Operations)
{
	FString Manifest1, Manifest2;
	if (FParse::Value(FCommandLine::Get(), TEXT("-oplog1="), Manifest1) && FParse::Value(FCommandLine::Get(), TEXT("-oplog2="), Manifest2))
	{
		TUniquePtr<IOplogDiffOperation> DiffOff = MakeUnique<FOplogManifestDiff>(Manifest1, Manifest2);
		DiffOff->OnComplete = [&Operations](IOplogDiffOperation& CompletedDiff)
		{
			FString ResultsPath;
			FParse::Value(FCommandLine::Get(), TEXT("-results-path="), ResultsPath);
			FOplogManifestDiff& DiffOp = static_cast<FOplogManifestDiff&>(CompletedDiff);
			TUniquePtr<FOutputDiffResults> OutputOp = MakeUnique<FOutputDiffResults>(MoveTemp(DiffOp.Manifest1),
				MoveTemp(DiffOp.Manifest2),
				MoveTemp(DiffOp.DiffResults));
			OutputOp->OutputPath = ResultsPath;
			OutputOp->LoggerOptions = GetDiffLogOptionsFromCmdLine();
			OutputOp->OutputOptions = GetDiffOutputOptionsFromCmdLine();
			Operations.Enqueue(MoveTemp(OutputOp));
		};
		Operations.Enqueue(MoveTemp(DiffOff));
	}
}

void QueueDownloadOperations(TQueue<TUniquePtr<IOplogDiffOperation>>& Operations, const TArray<UE::FZenSnapshotDescriptor>& Descriptors, bool bDownloadAsJson, FStringView OutputDirectory)
{
	for (const UE::FZenSnapshotDescriptor& Snapshot : Descriptors)
	{
		Operations.Enqueue(MakeUnique<FDownloadSnapshot>(Snapshot, bDownloadAsJson, OutputDirectory));
	}
}

void CreateDownloadCommands(TQueue<TUniquePtr<IOplogDiffOperation>>& Operations)
{
	FString SnapshotJsonPath;
	FString OutputDirectory;
	FString ProjectName;
	FString PlatformName;
	FString StreamName;
	FString CommitID;

	bool bDownloadAsJson = FParse::Param(FCommandLine::Get(), TEXT("-as-json"));
	FParse::Value(FCommandLine::Get(), TEXT("-output-directory="), OutputDirectory);
	if (FParse::Value(FCommandLine::Get(), TEXT("-snapshot-json="), SnapshotJsonPath))
	{
		// Load the descriptor, then queue the downloads
		TUniquePtr<FLoadSnapshotDescriptor> LoadSnapshot = MakeUnique<FLoadSnapshotDescriptor>(SnapshotJsonPath);
		LoadSnapshot->OnComplete = [&Operations, bDownloadAsJson, OutputDirectory](IOplogDiffOperation& CompletedLoad)
		{
			FLoadSnapshotDescriptor& LoadSnapshotOp = static_cast<FLoadSnapshotDescriptor&>(CompletedLoad);
			QueueDownloadOperations(Operations, LoadSnapshotOp.LoadedSnapshots, bDownloadAsJson, OutputDirectory);
		};
		Operations.Enqueue(MoveTemp(LoadSnapshot));
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("-project="), ProjectName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-stream="), StreamName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-platform="), PlatformName) &&
		FParse::Value(FCommandLine::Get(), TEXT("-commit="), CommitID))
	{
		// Find the snapshot, then queue the download
		TUniquePtr<FFindSnapshot> FindSnapshot = MakeUnique<FFindSnapshot>();
		FindSnapshot->FindPlatform = PlatformName;
		FindSnapshot->FindProjectName = ProjectName;
		FindSnapshot->FindStream = StreamName;
		FindSnapshot->FindCommitID = CommitID;
		FindSnapshot->OnComplete = [&Operations, bDownloadAsJson, OutputDirectory](IOplogDiffOperation& CompletedLoad)
		{
			FFindSnapshot& FindSnapshotOp = static_cast<FFindSnapshot&>(CompletedLoad);
			TArray<UE::FZenSnapshotDescriptor> Descriptors;
			Descriptors.Add(*FindSnapshotOp.FoundSnapshot);
			QueueDownloadOperations(Operations, Descriptors, bDownloadAsJson, OutputDirectory);
		};
		Operations.Enqueue(MoveTemp(FindSnapshot));
	}
}

void CreateInitialCommands(TQueue<TUniquePtr<IOplogDiffOperation>>& Operations)
{
	if (FParse::Param(FCommandLine::Get(), TEXT("-find-snapshot")))
	{
		CreateFindSnapshotCommands(Operations);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("-download-snapshot")))
	{
		CreateDownloadCommands(Operations);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("-diff-oplog")))
	{
		CreateManifestDiffCommands(Operations);
	}
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	const FTaskTagScope Scope(ETaskTag::EGameThread);

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	ON_SCOPE_EXIT
	{
		RequestEngineExit("Exiting");
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	// Don't allow more than one instance to run simultaneously 
	FSystemWideCriticalSection SystemWideCritSection(TEXT("ZenOplogDiffTool"));
	if (!SystemWideCritSection.IsValid())
	{
		UE_LOGF(LogZenOplogDiffTool, Warning, "Zen Oplog Diff is already running!");
		return 1;
	}

	// Disable display of log timestamps, categories and verbosity
	GLog->Flush();
	TGuardValue NoTime(GPrintLogTimes, ELogTimes::None);
	TGuardValue NoCategory(GPrintLogCategory, false);
	TGuardValue NoVerbosity(GPrintLogVerbosity, false);

	TQueue<TUniquePtr<IOplogDiffOperation>> AllOperations;	// Operations can be queued from other operations during the main loop!
	CreateInitialCommands(AllOperations);

	int32 ExitCode = 0;
	if (AllOperations.IsEmpty())
	{
		PrintHelp();
	}
	else
	{
		// Run a light version of the engine main loop
		TUniquePtr<IOplogDiffOperation> RunningOperation;
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();
		while (!IsEngineExitRequested())
		{
			BeginExitIfRequested();

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			if (RunningOperation == nullptr && !AllOperations.Dequeue(RunningOperation))
			{
				break;
			}
			else if (RunningOperation)
			{
				IOplogDiffOperation::ERunningState CurrentState = RunningOperation->Run();
				if (CurrentState != IOplogDiffOperation::ERunningState::Running)
				{
					if (CurrentState == IOplogDiffOperation::ERunningState::Success && RunningOperation->OnComplete)
					{
						RunningOperation->OnComplete(*RunningOperation);
					}
					else if (CurrentState == IOplogDiffOperation::ERunningState::Error)
					{
						ExitCode = 1;
						break;
					}
					RunningOperation = nullptr;
				}
			}

			DeltaTime = FPlatformTime::Seconds() - LastTime;
			LastTime = FPlatformTime::Seconds();

			UE::Stats::FStats::AdvanceFrame(false);
			FCoreDelegates::OnEndFrame.Broadcast();

			LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());

			GFrameCounter++;
		}
	}

	return ExitCode;
}
