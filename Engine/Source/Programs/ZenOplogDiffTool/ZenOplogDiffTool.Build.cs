// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ZenOplogDiffTool : ModuleRules
{
	public ZenOplogDiffTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("ApplicationCore");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("DesktopPlatform");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("ZenOplogUtils");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/crashpad_handler.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.pdb");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.pdb");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/win-x64/OidcToken.exe");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/linux-x64/OidcToken");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/crashpad_handler");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/osx-x64/OidcToken");
		}
	}
}
