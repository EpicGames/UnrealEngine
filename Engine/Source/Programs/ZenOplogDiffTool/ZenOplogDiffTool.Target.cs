// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class ZenOplogDiffToolTarget : TargetRules
{
	public ZenOplogDiffToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Modular;
		LaunchModuleName = "ZenOplogDiffTool";

		bBuildDeveloperTools = false;
		bCompileAgainstApplicationCore = true;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstEngine = false;
		bCompileICU = false;
		bForceDisableAutomationTests = true;
		bIsBuildingConsoleApplication = true;
		bUseLoggingInShipping = true;

		if (Target.Configuration == UnrealTargetConfiguration.Debug ||
			Target.Configuration == UnrealTargetConfiguration.Development)
		{
			GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
			GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
			bEnableTrace = true;
		}
	}
}
